// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorModeSceneDepthPicker.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SToolTip.h"
#include "LevelEditorViewport.h"
#include "EditorModes.h"
#include "Editor/ActorPositioning.h"
#include "Widgets/SWindow.h"
#include "SceneView.h"


#define LOCTEXT_NAMESPACE "SceneDepthPicker"

FEdModeSceneDepthPicker::FEdModeSceneDepthPicker()
{
	PickState = ESceneDepthPickState::NotOverViewport;
}

void FEdModeSceneDepthPicker::Initialize()
{
	CursorDecoratorWindow = SWindow::MakeCursorDecorator();
	FSlateApplication::Get().AddWindow(CursorDecoratorWindow.ToSharedRef(), true);
	CursorDecoratorWindow->SetContent(
		SNew(SToolTip)
		.Text(this, &FEdModeSceneDepthPicker::GetCursorDecoratorText)
		);
}

void FEdModeSceneDepthPicker::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	if(CursorDecoratorWindow.IsValid())
	{
		CursorDecoratorWindow->MoveWindowTo(FSlateApplication::Get().GetCursorPos() + FSlateApplication::Get().GetCursorSize());
	}

	FEdMode::Tick(ViewportClient, DeltaTime);
}

bool FEdModeSceneDepthPicker::MouseEnter(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y)
{
	PickState = ESceneDepthPickState::OverViewport;
	return FEdMode::MouseEnter(ViewportClient, Viewport, x, y);
}

bool FEdModeSceneDepthPicker::MouseLeave(FEditorViewportClient* ViewportClient, FViewport* Viewport)
{
	PickState = ESceneDepthPickState::NotOverViewport;
	return FEdMode::MouseLeave(ViewportClient, Viewport);
}

bool FEdModeSceneDepthPicker::MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y)
{
	if (ViewportClient == GCurrentLevelEditingViewportClient)
	{
		PickState = ESceneDepthPickState::OverViewport;
	}
	else
	{
		PickState = ESceneDepthPickState::NotOverViewport;
	}

	return true;
}

bool FEdModeSceneDepthPicker::LostFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport)
{
	if (ViewportClient == GCurrentLevelEditingViewportClient)
	{
		// Make sure actor picking mode is disabled once the active viewport loses focus
		RequestDeletion();
		return true;
	}

	return false;
}

bool FEdModeSceneDepthPicker::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	if (ViewportClient == GCurrentLevelEditingViewportClient)
	{
		if (Key == EKeys::LeftMouseButton && Event == IE_Pressed)
		{
			int32 const HitX = Viewport->GetMouseX();
			int32 const HitY = Viewport->GetMouseY();

			FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
				Viewport,
				ViewportClient->GetScene(),
				ViewportClient->EngineShowFlags)
				.SetRealtimeUpdate(true));
			FSceneView* View = ViewportClient->CalcSceneView(&ViewFamily);
			FViewportCursorLocation Cursor(View, ViewportClient, HitX, HitY);

			const FActorPositionTraceResult TraceResult = FActorPositioning::TraceWorldForPosition(Cursor, *View);

			FVector WorldPosition = TraceResult.State == FActorPositionTraceResult::HitSuccess ? TraceResult.Location : ViewportClient->GetHitProxyObjectLocation(HitX, HitY);

			OnSceneDepthLocationSelected.ExecuteIfBound(WorldPosition);
			RequestDeletion();
			return true;
		}
		else if(Key == EKeys::Escape && Event == IE_Pressed)
		{
			RequestDeletion();
			return true;
		}
	}
	else
	{
		RequestDeletion();
	}

	return false;
}

bool FEdModeSceneDepthPicker::GetCursor(EMouseCursor::Type& OutCursor) const
{
	if (PickState == ESceneDepthPickState::OverViewport)
	{
		OutCursor = EMouseCursor::EyeDropper;
	}
	else
	{
		OutCursor = EMouseCursor::SlashedCircle;
	}
	
	return true;
}

bool FEdModeSceneDepthPicker::UsesToolkits() const
{
	return false;
}

bool FEdModeSceneDepthPicker::IsCompatibleWith(FEditorModeID OtherModeID) const
{
	// We want to be able to perform this action with all the built-in editor modes
	return OtherModeID != FBuiltinEditorModes::EM_None;
}

void FEdModeSceneDepthPicker::Exit()
{
	OnSceneDepthLocationSelected = FOnSceneDepthLocationSelected();

	if (CursorDecoratorWindow.IsValid())
	{
		CursorDecoratorWindow->RequestDestroyWindow();
		CursorDecoratorWindow.Reset();
	}

	PickState = ESceneDepthPickState::NotOverViewport;

	FEdMode::Exit();
}

FText FEdModeSceneDepthPicker::GetCursorDecoratorText() const
{
	switch(PickState)
	{
	default:
	case ESceneDepthPickState::NotOverViewport:
		return LOCTEXT("PickSceneDepth_NotOverViewport", "Pick an location in an active level viewport to sample the depth");
	}
}



#undef LOCTEXT_NAMESPACE
