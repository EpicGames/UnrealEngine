// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "SceneDepthPickerMode.h"
#include "EdMode.h"

class FEditorViewportClient;
class FViewport;
class SWindow;

enum class ESceneDepthPickState : uint8
{
	NotOverViewport,
	OverViewport,
};

/**
 * Editor mode used to interactively pick actors in the level editor viewports.
 */
class FEdModeSceneDepthPicker : public FEdMode
{
public:
	FEdModeSceneDepthPicker();

	/** Begin FEdMode interface */
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	virtual bool MouseEnter(FEditorViewportClient* ViewportClient, FViewport* Viewport,int32 x, int32 y) override;
	virtual bool MouseLeave(FEditorViewportClient* ViewportClient, FViewport* Viewport) override;
	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override;
	virtual bool LostFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport) override;
	virtual bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;
	virtual bool GetCursor(EMouseCursor::Type& OutCursor) const override;
	virtual bool UsesToolkits() const override;
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override;
	virtual void Initialize() override;
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }
	/** End FEdMode interface */

	/** End the mode */
	virtual void Exit() override;

	/** Delegate used to display information about picking near the cursor */
	FText GetCursorDecoratorText() const;

	/** Flag for display state */
	ESceneDepthPickState PickState;

	/** The window that owns the decorator widget */
	TSharedPtr<SWindow> CursorDecoratorWindow;

	/** Delegates used to pick actors */
	FOnSceneDepthLocationSelected OnSceneDepthLocationSelected;
};
