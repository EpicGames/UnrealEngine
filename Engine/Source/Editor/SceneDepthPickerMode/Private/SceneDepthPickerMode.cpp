// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneDepthPickerMode.h"
#include "Modules/ModuleManager.h"
#include "EditorModeRegistry.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "EditorModeSceneDepthPicker.h"

IMPLEMENT_MODULE( FSceneDepthPickerModeModule, SceneDepthPickerMode );

void FSceneDepthPickerModeModule::StartupModule()
{
	FEditorModeRegistry::Get().RegisterMode<FEdModeSceneDepthPicker>(FBuiltinEditorModes::EM_SceneDepthPicker);
}

void FSceneDepthPickerModeModule::ShutdownModule()
{
	FEditorModeRegistry::Get().UnregisterMode(FBuiltinEditorModes::EM_SceneDepthPicker);
}

void FSceneDepthPickerModeModule::BeginSceneDepthPickingMode(FOnSceneDepthLocationSelected InOnSceneDepthLocationSelected)
{
	// Activate the mode
	GLevelEditorModeTools().ActivateMode(FBuiltinEditorModes::EM_SceneDepthPicker);

	// Set the required delegates
	FEdModeSceneDepthPicker* const Mode = GLevelEditorModeTools().GetActiveModeTyped<FEdModeSceneDepthPicker>(FBuiltinEditorModes::EM_SceneDepthPicker);
	if (ensure(Mode))
	{
		Mode->OnSceneDepthLocationSelected = InOnSceneDepthLocationSelected;
	}
}

void FSceneDepthPickerModeModule::EndSceneDepthPickingMode()
{
	GLevelEditorModeTools().DeactivateMode(FBuiltinEditorModes::EM_SceneDepthPicker);
}

bool FSceneDepthPickerModeModule::IsInSceneDepthPickingMode() const
{
	return GLevelEditorModeTools().IsModeActive(FBuiltinEditorModes::EM_SceneDepthPicker);
}
