// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#define UE_API SCENEDEPTHPICKERMODE_API

DECLARE_DELEGATE_OneParam( FOnSceneDepthLocationSelected, FVector );

/**
 * Scene depth picker mode module
 */
class FSceneDepthPickerModeModule : public IModuleInterface
{
public:
	// IModuleInterface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;
	// End of IModuleInterface

	/** 
	 * Enter scene depth picking mode (note: will cancel any current scene depth picking)
	 * @param	InOnDepthSelected		Delegate to call when user has selected a location to sample the scene depth
	 */
	UE_API void BeginSceneDepthPickingMode(FOnSceneDepthLocationSelected InOnSceneDepthLocationSelected);

	/** Exit scene depth picking mode */
	UE_API void EndSceneDepthPickingMode();

	/** @return Whether or not scene depth picking mode is currently active */
	UE_API bool IsInSceneDepthPickingMode() const;
};

#undef UE_API
